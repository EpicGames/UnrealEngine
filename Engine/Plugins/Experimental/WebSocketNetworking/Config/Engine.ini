;[/Script/Engine.GameEngine]
;!NetDriverDefinitions=ClearArray
;+NetDriverDefinitions=(DefName="GameNetDriver", DriverClassName="/Script/WebSocketNetworking.WebSocketNetDriver", DriverClassNameFallback="/Script/WebSocketNetworking.WebSocketNetDriver")
;[/Script/Engine.Engine]
;+NetDriverDefinitions=(DefName="GameNetDriver", DriverClassName="/Script/WebSocketNetworking.WebSocketNetDriver", DriverClassNameFallback="/Script/WebSocketsNetworking.IpNetDriver")

[/Script/WebSocketNetworking.WebSocketNetDriver]
AllowPeerConnections=False
AllowPeerVoice=False
ConnectionTimeout=60.0
InitialConnectTimeout=120.0
RecentlyDisconnectedTrackingTime=180
AckTimeout=10.0
KeepAliveTime=20.2
MaxClientRate=15000
MaxInternetClientRate=10000
RelevantTimeout=5.0
SpawnPrioritySeconds=1.0
ServerTravelPause=4.0
NetServerMaxTickRate=30
LanServerMaxTickRate=35
WebSocketPort=8889
NetConnectionClassName="/Script/WebSocketNetworking.WebSocketConnection"
MaxPortCountToTry=512

