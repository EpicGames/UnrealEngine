// Copyright Epic Games, Inc. All Rights Reserved.
#include "WebSocket.h"
#include "WebSocketNetworkingPrivate.h" // IWYU pragma: keep

#if USE_LIBWEBSOCKET

// Work around a conflict between a UI namespace defined by engine code and a typedef in OpenSSL
#define UI UI_ST
THIRD_PARTY_INCLUDES_START
#include "libwebsockets.h"
THIRD_PARTY_INCLUDES_END
#undef UI

static int unreal_networking_client(struct lws *wsi, enum lws_callback_reasons reason, void *user, void *in, size_t len);
static void lws_debugLogS(int level, const char *line)
{
	UE_LOGF(LogWebSocketNetworking, Log, "client: %ls", ANSI_TO_TCHAR(line));
}

#else

#include <errno.h>
#include <sys/types.h>
#include <sys/socket.h>
#include <netinet/in.h>
#include <arpa/inet.h>
#include <stdio.h>
#include <stdlib.h>
#include <string.h>
#include <unistd.h>
#include <fcntl.h>
#include <sys/ioctl.h>
#include <assert.h>
#include <emscripten/emscripten.h>

#endif

FWebSocket::FWebSocket(
		const FInternetAddr& ServerAddress
)
:IsServerSide(false)
{

#if USE_LIBWEBSOCKET

#if !UE_BUILD_SHIPPING
	lws_set_log_level(LLL_ERR | LLL_WARN | LLL_NOTICE | LLL_DEBUG | LLL_INFO, lws_debugLogS);
#endif

	Protocols = new lws_protocols[3];
	FMemory::Memzero(Protocols, sizeof(lws_protocols) * 3);

	Protocols[0].name = "binary";
	Protocols[0].callback = unreal_networking_client;
	Protocols[0].per_session_data_size = 0;
	Protocols[0].rx_buffer_size = 10 * 1024 * 1024;

	Protocols[1].name = nullptr;
	Protocols[1].callback = nullptr;
	Protocols[1].per_session_data_size = 0;

	struct lws_context_creation_info Info;
	memset(&Info, 0, sizeof Info);

	Info.port = CONTEXT_PORT_NO_LISTEN;
	Info.protocols = &Protocols[0];
	Info.gid = -1;
	Info.uid = -1;
	Info.user = this;

	//Info.options |= LWS_SERVER_OPTION_DO_SSL_GLOBAL_INIT;
	Info.options |= LWS_SERVER_OPTION_DISABLE_IPV6;

	Context = lws_create_context(&Info);

	check(Context);

	auto ServerAddressString = StringCast<ANSICHAR>(*ServerAddress.ToString(false));

	struct lws_client_connect_info ConnectInfo = {
			Context, ServerAddressString.Get(), ServerAddress.GetPort(), false, "/", ServerAddressString.Get(), ServerAddressString.Get(), Protocols[1].name, -1, this
	};
	Wsi = lws_client_connect_via_info(&ConnectInfo);
	check(Wsi);

#else // ! USE_LIBWEBSOCKET -- HTML5 uses BSD network API

	SockFd = socket(PF_INET, SOCK_STREAM, IPPROTO_TCP);
	if (SockFd == -1)
	{
		UE_LOGF(LogWebSocketNetworking, Error, "Socket creation failed ");
	}
	else
	{
		UE_LOGF(LogWebSocketNetworking, Warning, " Socked %d created ", SockFd);
	}

	fcntl(SockFd, F_SETFL, O_NONBLOCK);

#endif

	memset(&RemoteAddr, 0, sizeof(RemoteAddr));

	// Windows XP does not have support for inet_pton
#if PLATFORM_WINDOWS && _WIN32_WINNT <= 0x0502
	int32 SizeOfRemoteAddr = sizeof(RemoteAddr);

	// Force ServerAddress into non-const array. API doesn't modify contents but old API still requires non-const string
	if (WSAStringToAddress(ServerAddress.ToString(false).GetCharArray().GetData(), AF_INET, NULL, (sockaddr*)&RemoteAddr, &SizeOfRemoteAddr) != 0)
	{
		UE_LOGF(LogWebSocketNetworking, Warning, "WSAStringToAddress failed ");
		return;
	}
#else
	if (inet_pton(AF_INET, TCHAR_TO_ANSI(*ServerAddress.ToString(false)), &RemoteAddr.sin_addr) != 1)
	{
		UE_LOGF(LogWebSocketNetworking, Warning, "inet_pton failed to %ls", *ServerAddress.ToString(false));
		return;
	}
#endif

	RemoteAddr.sin_family = AF_INET;
	RemoteAddr.sin_port = htons(ServerAddress.GetPort());

#if !USE_LIBWEBSOCKET // HTML5 uses BSD network API
	int Ret = connect(SockFd, (struct sockaddr *)&RemoteAddr, sizeof(RemoteAddr));
	UE_LOGF(LogWebSocketNetworking, Warning, " Connect socket returned %d to %ls. Error Code: %d", Ret, *ServerAddress.ToString(false), ((Ret != 0) ? errno : 0));
#endif
}

#if USE_LIBWEBSOCKET
FWebSocket::FWebSocket(WebSocketInternalContext* InContext, WebSocketInternal* InWsi)
	: Context(InContext)
	, Wsi(InWsi)
	, Protocols(nullptr)
	, IsServerSide(true)
{
	int sock = lws_get_socket_fd(Wsi);
	socklen_t len = sizeof RemoteAddr;
	getpeername(sock, (struct sockaddr*)&RemoteAddr, &len);
}
#endif

bool FWebSocket::Send(const uint8* Data, uint32 Size, bool bPrependSize)
{
	TArray<uint8> Buffer;

#if USE_LIBWEBSOCKET
	Buffer.AddDefaulted(LWS_PRE); // Reserve space for WS header data
#endif

	if (bPrependSize)
	{
		Buffer.Append((uint8*)&Size, sizeof(uint32)); // insert size.
	}

	Buffer.Append((uint8*)Data, Size);
	OutgoingBuffer.Add(Buffer);

	return true;
}

void FWebSocket::SetReceiveCallBack(FWebSocketPacketReceivedCallBack CallBack)
{
	ReceivedCallback = CallBack;
}

void FWebSocket::SetSocketClosedCallBack(FWebSocketInfoCallBack CallBack)
{
	SocketClosedCallback = CallBack;
}

FString FWebSocket::RemoteEndPoint(bool bAppendPort)
{
	// Windows XP does not have support for inet_ntop
#if PLATFORM_WINDOWS && _WIN32_WINNT <= 0x0502
	TCHAR Buffer[INET_ADDRSTRLEN];
	::DWORD BufferSize = INET_ADDRSTRLEN;
	FString remote = "";
	sockaddr_in AddressToConvert = RemoteAddr;
	if (!bAppendPort)
	{
		AddressToConvert.sin_port = 0;
	}
	if (WSAAddressToString((sockaddr*)&AddressToConvert, sizeof(AddressToConvert), NULL, Buffer, &BufferSize) == 0)
	{
		remote = Buffer;
	}
#else
	ANSICHAR Buffer[INET_ADDRSTRLEN];
	FString remote(ANSI_TO_TCHAR(inet_ntop(AF_INET, &RemoteAddr.sin_addr, Buffer, INET_ADDRSTRLEN)));
	if ( bAppendPort )
	{
		remote += FString::Printf(TEXT(":%i"), ntohs(RemoteAddr.sin_port));
	}
#endif

	return remote;
}

FString FWebSocket::LocalEndPoint(bool bAppendPort)
{
#if USE_LIBWEBSOCKET
	int sock = lws_get_socket_fd(Wsi);
	struct sockaddr_in addr;
	socklen_t len = sizeof addr;
	getsockname(sock, (struct sockaddr*)&addr, &len);

	// Windows XP does not have support for inet_ntop
#if PLATFORM_WINDOWS && _WIN32_WINNT <= 0x0502
	TCHAR Buffer[INET_ADDRSTRLEN];
	::DWORD BufferSize = INET_ADDRSTRLEN;
	FString remote = "";
	if (!bAppendPort)
	{
		addr.sin_port = 0;
	}
	if (WSAAddressToString((sockaddr*)&addr, sizeof(addr), NULL, Buffer, &BufferSize) == 0)
	{
		remote = Buffer;
	}
#else
	ANSICHAR Buffer[INET_ADDRSTRLEN];
	FString remote(ANSI_TO_TCHAR(inet_ntop(AF_INET, &addr.sin_addr, Buffer, INET_ADDRSTRLEN)));
	if ( bAppendPort )
	{
		remote += FString::Printf(TEXT(":%i"), ntohs(addr.sin_port));
	}
#endif

	return remote;
#else // ! USE_LIBWEBSOCKET -- HTML5 uses BSD network API
	// NOTE: there's no way to get this info from browsers...
	// return generic localhost without port #
	return FString(TEXT("127.0.0.1"));
#endif
}

TArray<FString> FWebSocket::GetUrlArgs() 
{
	TArray<FString> UrlArgs;
#if USE_LIBWEBSOCKET
	char Buffer[256];	
	
	int n = 0;
	while ( lws_hdr_copy_fragment(Wsi, Buffer, sizeof(Buffer), WSI_TOKEN_HTTP_URI_ARGS, n) >= 0 ) 
	{
		UrlArgs.Add(FString(ANSI_TO_TCHAR(Buffer)));
		n++;
	}
#endif

	return UrlArgs;	
}

FString FWebSocket::GetUrlArgByName(const FString& Name) 
{
#if USE_LIBWEBSOCKET
	char Buffer[256];	

	if ( lws_get_urlarg_by_name(Wsi, TCHAR_TO_ANSI(*Name), Buffer, sizeof(Buffer)) != nullptr )
	{
		return FString(ANSI_TO_TCHAR(Buffer));
	}
#endif

	return FString(TEXT(""));
}

void FWebSocket::Tick()
{
	HandlePacket();
}

void FWebSocket::HandlePacket()
{
#if USE_LIBWEBSOCKET

	lws_service(Context, 0);
	if (!IsServerSide)
		lws_callback_on_writable_all_protocol(Context, &Protocols[0]);

#else // ! USE_LIBWEBSOCKET -- HTML5 uses BSD network API

	fd_set Fdr;
	fd_set Fdw;
	int Res;

	// make sure that server.fd is ready to read / write
	FD_ZERO(&Fdr);
	FD_ZERO(&Fdw);
	FD_SET(SockFd, &Fdr);
	FD_SET(SockFd, &Fdw);
	Res = select(64, &Fdr, &Fdw, NULL, NULL);

	if (Res == -1) {
		UE_LOGF(LogWebSocketNetworking, Warning, "Select Failed!");
		return;
	}

	if (FD_ISSET(SockFd, &Fdr)) {
		// we can read!
		OnRawRecieve(NULL, NULL);
	}

	if (FD_ISSET(SockFd, &Fdw)) {
		// we can write
		OnRawWebSocketWritable(NULL);
	}

#endif
}

void FWebSocket::Flush()
{
	auto PendingMesssages = OutgoingBuffer.Num();
	while (OutgoingBuffer.Num() > 0 && !IsServerSide)
	{
#if USE_LIBWEBSOCKET
		if (Protocols)
		{
			lws_callback_on_writable_all_protocol(Context, &Protocols[0]);
		}
		else
		{
			lws_callback_on_writable(Wsi);
		}
#endif
		HandlePacket();
		if (PendingMesssages >= OutgoingBuffer.Num())
		{
			UE_LOGF(LogWebSocketNetworking, Warning, "Unable to flush all of OutgoingBuffer in FWebSocket.");
			break;
		}
	};
}

TArray<uint8> FWebSocket::GetRawRemoteAddr(int32& OutPort)
{
	OutPort = ntohs(RemoteAddr.sin_port);
	TArray<uint8> RawBuffer;
	uint32 IntAddr = RemoteAddr.sin_addr.s_addr;
	RawBuffer.Add((IntAddr >> 0) & 0xFF);
	RawBuffer.Add((IntAddr >> 8) & 0xFF);
	RawBuffer.Add((IntAddr >> 16) & 0xFF);
	RawBuffer.Add((IntAddr >> 24) & 0xFF);

	return RawBuffer;
}

void FWebSocket::SetConnectedCallBack(FWebSocketInfoCallBack CallBack)
{
	ConnectedCallBack = CallBack;
}

void FWebSocket::SetErrorCallBack(FWebSocketInfoCallBack CallBack)
{
	ErrorCallBack = CallBack;
}

void FWebSocket::OnReceive(void* Data, uint32 Size)
{
#if USE_LIBWEBSOCKET
	ReceivedCallback.ExecuteIfBound(Data, Size);
#endif
}

void FWebSocket::OnRawRecieve(void* Data, uint32 Size)
{
#if USE_LIBWEBSOCKET

	ReceiveBuffer.Append((uint8*)Data, Size); // consumes all of Data
	while (ReceiveBuffer.Num() > sizeof(uint32))
	{
		uint32 BytesToBeRead = *(uint32*)ReceiveBuffer.GetData();
		if (BytesToBeRead <= ((uint32)ReceiveBuffer.Num() - sizeof(uint32)))
		{
			ReceivedCallback.ExecuteIfBound((void*)((uint8*)ReceiveBuffer.GetData() + sizeof(uint32)), BytesToBeRead);
			ReceiveBuffer.RemoveAt(0, sizeof(uint32) + BytesToBeRead );
		}
		else
		{
			break;
		}
	}

#else // ! USE_LIBWEBSOCKET -- HTML5 uses BSD network API

	// browser was crashing when using ReceiveBuffer...

	check(Data == NULL); // jic this is not obvious, Data will be resigned to Buffer below

	uint8 Buffer[1024]; // should be at MAX PACKET SIZE.
	Size = recv(SockFd, Buffer, sizeof(Buffer), 0);
//	while ( Size > sizeof(uint32) )
	if ( Size > sizeof(uint32) )
	{
		Data = (void*)Buffer;
		uint32 BytesToBeRead = *(uint32*)Data;
		uint32 BytesLeft = Size;
		while ( ( BytesToBeRead > 0 ) && ( BytesToBeRead < BytesLeft ) )
		{
			Data = (void*)((uint8*)Data + sizeof(uint32));
			ReceivedCallback.ExecuteIfBound(Data, BytesToBeRead);

			// "ReceiveBuffer.RemoveAt()"
			Data = (void*)((uint8*)Data + BytesToBeRead);
			BytesLeft -= BytesToBeRead + sizeof(uint32);
			if ( (uint8*)Data >= (Buffer+Size)  // hard cap
				||	BytesLeft == 0	)           // soft limit
			{
				break;
			}
			BytesToBeRead = *(uint32*)Data;
		}
//		Size = recv(SockFd, Buffer, sizeof(Buffer), 0);
	}

#endif
}

void FWebSocket::OnRawWebSocketWritable(WebSocketInternal* wsi)
{
	if (OutgoingBuffer.Num() == 0)
		return;

	TArray <uint8>& Packet = OutgoingBuffer[0];

#if USE_LIBWEBSOCKET

	uint32 TotalDataSize = Packet.Num() - LWS_PRE;
	uint32 DataToSend = TotalDataSize;
	while (DataToSend)
	{
		int Sent = lws_write(Wsi, Packet.GetData() + LWS_PRE + (DataToSend-TotalDataSize), DataToSend, (lws_write_protocol)LWS_WRITE_BINARY);
		if (Sent < 0)
		{
			ErrorCallBack.ExecuteIfBound();
			return;
		}
		if ((uint32)Sent < DataToSend)
		{
			UE_LOGF(LogWebSocketNetworking, Warning, "Could not write all '%d' bytes to socket", DataToSend);
		}
		DataToSend-=Sent;
	}

	check(Wsi == wsi);

#else // ! USE_LIBWEBSOCKET -- HTML5 uses BSD network API

	uint32 TotalDataSize = Packet.Num();
	uint32 DataToSend = TotalDataSize;
	while (DataToSend)
	{
		// send actual data in one go.
		int Result = send(SockFd, Packet.GetData()+(DataToSend-TotalDataSize),DataToSend, 0);
		if (Result == -1)
		{
			// we are caught with our pants down. fail.
			UE_LOGF(LogWebSocketNetworking, Error, "Could not write %d bytes", Packet.Num());
			ErrorCallBack.ExecuteIfBound();
			return;
		}
		UE_CLOG((uint32)Result < DataToSend, LogWebSocketNetworking, Warning, TEXT("Could not write all '%d' bytes to socket"), DataToSend);
		DataToSend-=Result;
	}

#endif

	// this is very inefficient we need a constant size circular buffer to efficiently not do unnecessary allocations/deallocations.
	OutgoingBuffer.RemoveAt(0);

}

void FWebSocket::OnClose()
{
	SocketClosedCallback.ExecuteIfBound();
}

FWebSocket::~FWebSocket()
{
	ReceivedCallback.Unbind();

#if USE_LIBWEBSOCKET

	Flush();

	if ( !IsServerSide)
	{
		lws_context_destroy(Context);
		Context = NULL;
		delete Protocols;
		Protocols = NULL;
	}

#else // ! USE_LIBWEBSOCKET -- HTML5 uses BSD network API

	close(SockFd);

#endif

}

#if USE_LIBWEBSOCKET
static int unreal_networking_client(
		struct lws *Wsi,
		enum lws_callback_reasons Reason,
		void *User,
		void *In,
		size_t Len)
{
	struct lws_context *Context = lws_get_context(Wsi);
	FWebSocket* Socket = (FWebSocket*)lws_context_user(Context);
	switch (Reason)
	{
	case LWS_CALLBACK_CLIENT_ESTABLISHED:
		{
			Socket->ConnectedCallBack.ExecuteIfBound();
			lws_set_timeout(Wsi, NO_PENDING_TIMEOUT, 0);
			check(Socket->Wsi == Wsi);
		}
		break;
	case LWS_CALLBACK_CLIENT_CONNECTION_ERROR:
		{
			Socket->ErrorCallBack.ExecuteIfBound();
			return -1;
		}
	case LWS_CALLBACK_CLIENT_RECEIVE:
		{
			// push it on the socket.
			Socket->OnRawRecieve(In, (uint32)Len);
			check(Socket->Wsi == Wsi);
			lws_set_timeout(Wsi, NO_PENDING_TIMEOUT, 0);
			break;
		}
	case LWS_CALLBACK_CLIENT_WRITEABLE:
		{
			check(Socket->Wsi == Wsi);
			Socket->OnRawWebSocketWritable(Wsi);
			lws_callback_on_writable(Wsi);
			lws_set_timeout(Wsi, NO_PENDING_TIMEOUT, 0);
			break;
		}
	case LWS_CALLBACK_CLIENT_CLOSED:
		{
			Socket->ErrorCallBack.ExecuteIfBound();
			return -1;
		}
	}

	return 0;
}
#endif
