// Copyright Epic Games, Inc. All Rights Reserved.
//
// libwebsocket client wrapper.
//
#pragma  once
#include "INetworkingWebSocket.h"
#include "WebSocketNetworkingPrivate.h"
#if USE_LIBWEBSOCKET
#include "Runtime/Sockets/Private/BSDSockets/SocketSubsystemBSD.h" // IWYU pragma: keep
#else
#include <netinet/in.h>
#endif

class FWebSocket : public INetworkingWebSocket
{

public:

	// Initialize as client side socket.
	FWebSocket(const FInternetAddr& ServerAddress);

#if USE_LIBWEBSOCKET
	// Initialize as server side socket.
	FWebSocket(WebSocketInternalContext* InContext, WebSocketInternal* Wsi);
#endif

	//~ Begin INetworkingWebSocket interface
	virtual ~FWebSocket() override;	
	virtual void SetConnectedCallBack(FWebSocketInfoCallBack CallBack) override;
	virtual void SetErrorCallBack(FWebSocketInfoCallBack CallBack) override;
	virtual void SetReceiveCallBack(FWebSocketPacketReceivedCallBack CallBack) override;
	virtual void SetSocketClosedCallBack(FWebSocketInfoCallBack CallBack) override;
	virtual bool Send(const uint8* Data, uint32 Size, bool bPrependSize = true) override;
	virtual void Tick() override;
	virtual void Flush() override;
	virtual TArray<uint8> GetRawRemoteAddr(int32& OutPort) override;
	virtual FString RemoteEndPoint(bool bAppendPort) override;
	virtual FString LocalEndPoint(bool bAppendPort) override;
	virtual struct sockaddr_in* GetRemoteAddr() override { return &RemoteAddr; }
	virtual TArray<FString> GetUrlArgs() override;
	virtual FString GetUrlArgByName(const FString& Name) override;
	//~ End INetworkingWebSocket interface

// this was made public because of cross-platform build issues
public:
	void HandlePacket();
	void OnReceive(void* Data, uint32 Size);
	void OnRawRecieve(void* Data, uint32 Size);
	void OnRawWebSocketWritable(WebSocketInternal* wsi);
	void OnClose();

	/************************************************************************/
	/*	Various Socket callbacks											*/
	/************************************************************************/
	FWebSocketPacketReceivedCallBack  ReceivedCallback;
	FWebSocketInfoCallBack ConnectedCallBack;
	FWebSocketInfoCallBack ErrorCallBack;
	FWebSocketInfoCallBack SocketClosedCallback;

	/**  Recv and Send Buffers, serviced during the Tick */
	TArray<uint8> ReceiveBuffer;
	TArray<TArray<uint8>> OutgoingBuffer;

#if USE_LIBWEBSOCKET
	/** libwebsocket internal context*/
	WebSocketInternalContext* Context;

	/** libwebsocket web socket */
	WebSocketInternal* Wsi;

	/** libwebsocket Protocols that can be serviced by this implemenation*/
	WebSocketInternalProtocol* Protocols;
#else // ! USE_LIBWEBSOCKET -- HTML5 uses BSD network API
	int SockFd;
#endif
	struct sockaddr_in RemoteAddr;

	/** Server side socket or client side*/
	bool IsServerSide;

	friend class FWebSocketServer;
};
