// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Unreal WebSocket network driver.
=============================================================================*/

#include "WebSocketNetDriver.h"


#include "Engine/NetworkDelegates.h"
#include "WebSocketConnection.h"
#include "WebSocketServer.h"
#include "WebSocket.h"

#include "Misc/CommandLine.h"
#include "PacketHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WebSocketNetDriver)

/*-----------------------------------------------------------------------------
Declarations.
-----------------------------------------------------------------------------*/

/** Size of the network recv buffer */
#define NETWORK_MAX_PACKET (576)

UWebSocketNetDriver::UWebSocketNetDriver(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UWebSocketNetDriver::IsAvailable() const
{
	// WebSocket driver always valid for now
	return true;
}

ISocketSubsystem* UWebSocketNetDriver::GetSocketSubsystem()
{
	return ISocketSubsystem::Get();
}

bool UWebSocketNetDriver::InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error)
{
	if (!Super::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		return false;
	}

	return true;
}

bool UWebSocketNetDriver::InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error)
{
	if (!InitBase(true, InNotify, ConnectURL, false, Error))
	{
		return false;
	}

	// Create new connection.
	ServerConnection = NewObject<UWebSocketConnection>(NetConnectionClass);

	TSharedRef<FInternetAddr> InternetAddr = GetSocketSubsystem()->CreateInternetAddr();
	bool Ok;
	InternetAddr->SetIp(*ConnectURL.Host, Ok);
	InternetAddr->SetPort(WebSocketPort);

	FWebSocket* WebSocket = new FWebSocket(*InternetAddr);
	UWebSocketConnection* Connection = (UWebSocketConnection*)ServerConnection;
	Connection->SetWebSocket(WebSocket);

	FWebSocketPacketReceivedCallBack CallBack;
	CallBack.BindUObject(Connection, &UWebSocketConnection::ReceivedRawPacket);
	WebSocket->SetReceiveCallBack(CallBack);

	FWebSocketInfoCallBack  ConnectedCallBack;
	ConnectedCallBack.BindUObject(this, &UWebSocketNetDriver::OnWebSocketServerConnected);
	WebSocket->SetConnectedCallBack(ConnectedCallBack);

	ServerConnection->InitLocalConnection(this, NULL, ConnectURL, USOCK_Pending);

	// Create channel zero.
	CreateInitialClientChannels();

	return true;
}

bool UWebSocketNetDriver::InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error)
{
	if (!InitBase(false, InNotify, LocalURL, bReuseAddressAndPort, Error))
	{
		return false;
	}

	InitConnectionlessHandler();

	WebSocketServer = new class FWebSocketServer();

	FWebSocketClientConnectedCallBack CallBack;
	CallBack.BindUObject(this, &UWebSocketNetDriver::OnWebSocketClientConnected);

	if(!WebSocketServer->Init(WebSocketPort, CallBack))
		return false;

	WebSocketServer->Tick();
	LocalURL.Port = WebSocketPort;
	LocalAddr = GetSocketSubsystem()->GetLocalBindAddr(*GLog);
	LocalAddr->SetPort(WebSocketPort);
	UE_LOGF(LogWebSocketNetworking, Log, "%ls WebSocketNetDriver listening on port %i", *GetDescription(), LocalURL.Port);

	// server has no server connection.
	ServerConnection = NULL;
	return true;
}

void UWebSocketNetDriver::TickDispatch(float DeltaTime)
{
	Super::TickDispatch(DeltaTime);

	if (WebSocketServer)
		WebSocketServer->Tick();
}

void UWebSocketNetDriver::LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	if (Address.IsValid() && Address->IsValid())
	{
		const uint8* DataToSend = reinterpret_cast<uint8*>(Data);

		if (ConnectionlessHandler.IsValid())
		{
			const ProcessedPacket ProcessedData =
					ConnectionlessHandler->OutgoingConnectionless(Address, (uint8*)DataToSend, CountBits, Traits);

			if (!ProcessedData.bError)
			{
				DataToSend = ProcessedData.Data;
				CountBits = ProcessedData.CountBits;
			}
			else
			{
				CountBits = 0;
			}
		}


		// connectionless websockets do not exist (yet)
		// scan though existing connections
		if (CountBits > 0)
		{
			for (int32 i = 0; i<ClientConnections.Num(); ++i)
			{
				UWebSocketConnection* Connection = (UWebSocketConnection*)ClientConnections[i];
				if (Connection && ( Connection->LowLevelGetRemoteAddress(true) == Address->ToString(true) ) )
				{
					Connection->GetWebSocket()->Send((uint8*)DataToSend, FMath::DivideAndRoundUp(CountBits, 8));
					break;
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogNet, Warning, "UWebSocketNetDriver::LowLevelSend: Invalid send address '%ls'", *Address->ToString(false));
	}
}

FString UWebSocketNetDriver::LowLevelGetNetworkNumber()
{
	return WebSocketServer != nullptr ? WebSocketServer->Info() : FString(TEXT(""));
}

void UWebSocketNetDriver::LowLevelDestroy()
{
	Super::LowLevelDestroy();
	delete WebSocketServer;
	WebSocketServer = nullptr;
}

bool UWebSocketNetDriver::HandleSocketsCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld)
{
	Ar.Logf(TEXT(""));
	if (WebSocketServer != NULL)
	{
		Ar.Logf(TEXT("Running WebSocket Server %s"), *WebSocketServer->Info());
	}
	else
	{
		check(GetServerConnection());
		Ar.Logf(TEXT("WebSocket client's EndPoint %s"), *(GetServerConnection()->WebSocket->RemoteEndPoint(true)));
	}
	return UNetDriver::Exec(InWorld, TEXT("SOCKETS"), Ar);
}

#if UE_ALLOW_EXEC_COMMANDS
bool UWebSocketNetDriver::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("SOCKETS")))
	{
		return HandleSocketsCommand(Cmd, Ar, InWorld);
	}
	return UNetDriver::Exec(InWorld, Cmd, Ar);
}
#endif // UE_ALLOW_EXEC_COMMANDS

UWebSocketConnection* UWebSocketNetDriver::GetServerConnection()
{
	return (UWebSocketConnection*)ServerConnection;
}

void UWebSocketNetDriver::OnWebSocketClientConnected(INetworkingWebSocket* ClientWebSocket)
{
	// Determine if allowing for client/server connections
	const bool bAcceptingConnection = Notify != nullptr && Notify->NotifyAcceptingConnection() == EAcceptConnection::Accept;
	if (bAcceptingConnection)
	{

		UWebSocketConnection* Connection = NewObject<UWebSocketConnection>(NetConnectionClass);
		check(Connection);

		TSharedRef<FInternetAddr> InternetAddr = GetSocketSubsystem()->CreateInternetAddr();
		int32 AddressPort = 0;

		InternetAddr->SetRawIp(ClientWebSocket->GetRawRemoteAddr(AddressPort));
		InternetAddr->SetPort(0);
		Connection->SetWebSocket(ClientWebSocket);
		Connection->InitRemoteConnection(this, NULL, FURL(), *InternetAddr, USOCK_Open);
		
		Notify->NotifyAcceptedConnection(Connection);

		AddClientConnection(Connection);

		FWebSocketPacketReceivedCallBack CallBack;
		CallBack.BindUObject(Connection, &UWebSocketConnection::ReceivedRawPacket);
		if (ConnectionlessHandler.IsValid() && StatelessConnectComponent.IsValid())
		{
			Connection->bChallengeHandshake = true;
		}
#if !UE_BUILD_SHIPPING
		else if (FParse::Param(FCommandLine::Get(), TEXT("NoPacketHandler")))
		{
			UE_LOGF(LogNet, Log, "Accepting connection without handshake, due to '-NoPacketHandler'.")
		}
#endif
		else
		{
			UE_LOGF(LogNet, Log,
					"Invalid ConnectionlessHandler (%i) or StatelessConnectComponent (%i); can't accept connections.",
					(int32)(ConnectionlessHandler.IsValid()), (int32)(StatelessConnectComponent.IsValid()));
		}
		ClientWebSocket->SetReceiveCallBack(CallBack);

		UE_LOGF(LogWebSocketNetworking, Log, " WebSocket server running on %ls Accepted Connection from %ls ", *WebSocketServer->Info(),*ClientWebSocket->RemoteEndPoint(true));
	}
}

bool UWebSocketNetDriver::IsNetResourceValid(void)
{
	if (	(WebSocketServer && !ServerConnection)//  Server
		||	(!WebSocketServer && ServerConnection) // client
		)
	{
		return true;
	}
		
	return false;
}

// Just logging, not yet attached to html5 clients.
void UWebSocketNetDriver::OnWebSocketServerConnected()
{
	check(GetServerConnection());
	UE_LOGF(LogWebSocketNetworking, Log, " %ls WebSocket Client %ls connected to server %ls ", *GetDescription(),
		*GetServerConnection()->WebSocket->LocalEndPoint(true),
		*GetServerConnection()->WebSocket->RemoteEndPoint(true));
}

