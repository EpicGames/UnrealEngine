// Copyright Epic Games, Inc. All Rights Reserved.

#include "IWebSocketNetworkingModule.h"
#include "WebSocketServer.h"
#include "Modules/ModuleManager.h"
#include "WebSocket.h"



class FWebSocketNetworkingPlugin : public IWebSocketNetworkingModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}


public:
	TUniquePtr<IWebSocketServer> CreateServer() override
	{
		return MakeUnique<FWebSocketServer>();
	}

	TSharedPtr<INetworkingWebSocket> CreateConnection(const FInternetAddr& ServerAddress) override
	{
		return MakeShared<FWebSocket>(ServerAddress);
	}

};

IMPLEMENT_MODULE(FWebSocketNetworkingPlugin, WebSocketNetworking)

DEFINE_LOG_CATEGORY(LogWebSocketNetworking);


