// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Logging/LogMacros.h"

class FWebSocket; 
class FWebSocketServer; 

typedef struct lws_context WebSocketInternalContext;
typedef struct lws WebSocketInternal;
typedef struct lws_protocols WebSocketInternalProtocol;
typedef struct lws_http_mount WebSocketInternalHttpMount;

DECLARE_LOG_CATEGORY_EXTERN(LogWebSocketNetworking, Warning, All);
