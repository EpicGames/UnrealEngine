// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketServer.h"

#include "WebSocket.h"

#if USE_LIBWEBSOCKET

	#if PLATFORM_WINDOWS
		#include "Windows/AllowWindowsPlatformTypes.h"
		#include <winsock2.h>
		#include "Windows/HideWindowsPlatformTypes.h"
	#endif

	// Work around a conflict between a UI namespace defined by engine code and a typedef in OpenSSL
	#define UI UI_ST
THIRD_PARTY_INCLUDES_START
	#include "libwebsockets.h"
THIRD_PARTY_INCLUDES_END
	#undef UI
#endif

// The current state of the message being read.
enum class EFragmentationState : uint8
{
	BeginFrame,
	MessageFrame,
};

// An object of this type is associated by libwebsocket to every connected session.
struct PerSessionDataServer
{
	// Each session is actually a socket to a client
	FWebSocket* Socket;
	// Holds the concatenated message fragments.er
	TArray<uint8> FrameBuffer;
	// The current state of the message being read.
	EFragmentationState FragementationState = EFragmentationState::BeginFrame;
};

#if USE_LIBWEBSOCKET
// real networking handler.
static int unreal_networking_server(struct lws* wsi, enum lws_callback_reasons reason, void* user, void* in, size_t len);

	#if !UE_BUILD_SHIPPING
inline void lws_debugLog(int level, const char* line)
{
	UE_LOGF(LogWebSocketNetworking, Log, "websocket server: %ls", ANSI_TO_TCHAR(line));
}
	#endif // UE_BUILD_SHIPPING

#endif // USE_LIBWEBSOCKET

bool FWebSocketServer::IsHttpEnabled() const
{
	return bEnableHttp;
}

void FWebSocketServer::EnableHTTPServer(TArray<FWebSocketHttpMount> InDirectoriesToServe, bool bInSecure, const FWebSocketServerCertificates& InCertificates)
{
#if USE_LIBWEBSOCKET
	bEnableHttp = true;
	bSecure = bInSecure;
	Certificates = InCertificates;

	DirectoriesToServe = MoveTemp(InDirectoriesToServe);
	const int NMounts = DirectoriesToServe.Num();

	if (NMounts == 0)
	{
		return;
	}

	// Convert DirectoriesToServe to lws_http_mount for lws
	LwsHttpMounts = new lws_http_mount[NMounts];

	for (int i = 0; i < NMounts; i++)
	{
		bool						bLastMount = i == (NMounts - 1);
		FWebSocketHttpMount&		MountDir = DirectoriesToServe[i];
		WebSocketInternalHttpMount* LWSMount = &LwsHttpMounts[i];
		LWSMount->mount_next = bLastMount ? NULL : &LwsHttpMounts[i + 1];
		LWSMount->mountpoint = MountDir.GetWebPath();
		LWSMount->origin = MountDir.GetPathOnDisk();

		if (!MountDir.HasDefaultFile())
		{
			LWSMount->def = MountDir.GetDefaultFile();
		}
		else
		{
			LWSMount->def = NULL;
		}

		LWSMount->protocol = NULL;
		LWSMount->cgienv = NULL;
		LWSMount->extra_mimetypes = NULL; // We may wish to expose this in future
		LWSMount->interpret = NULL;
		LWSMount->cgi_timeout = 0;
		LWSMount->cache_max_age = 0;
		LWSMount->auth_mask = 0;
		LWSMount->cache_reusable = 0;
		LWSMount->cache_revalidate = 0;
		LWSMount->cache_intermediaries = 0;
		LWSMount->origin_protocol = LWSMPRO_FILE;
		LWSMount->mountpoint_len = FPlatformString::Strlen(MountDir.GetWebPath());
		LWSMount->basic_auth_login_file = NULL;
	}
#endif
}

bool FWebSocketServer::Init(uint32 Port, FWebSocketClientConnectedCallBack CallBack, FString InBindAddress)
{
#if USE_LIBWEBSOCKET
	#if !UE_BUILD_SHIPPING
	// setup log level.
	lws_set_log_level(LLL_ERR | LLL_WARN | LLL_NOTICE | LLL_DEBUG | LLL_INFO, lws_debugLog);
	#endif

	Protocols = new lws_protocols[3];
	FMemory::Memzero(Protocols, sizeof(lws_protocols) * 3);

	Protocols[0].name = "binary";
	Protocols[0].callback = unreal_networking_server;
	Protocols[0].per_session_data_size = sizeof(PerSessionDataServer);

	#if PLATFORM_WINDOWS
	Protocols[0].rx_buffer_size = 10 * 1024 * 1024;
	#else
	Protocols[0].rx_buffer_size = 1 * 1024 * 1024;
	#endif

	Protocols[1].name = nullptr;
	Protocols[1].callback = nullptr;
	Protocols[1].per_session_data_size = 0;

	struct lws_context_creation_info Info;
	memset(&Info, 0, sizeof(lws_context_creation_info));
	// look up libwebsockets.h for details.
	Info.port = Port;
	ServerPort = Port;

	Info.iface = NULL;
	if (!InBindAddress.IsEmpty())
	{
		size_t AddressLen = FCStringAnsi::Strlen(StringCast<ANSICHAR>(*InBindAddress).Get());
		BindAddress = new char[AddressLen + 1];
		FCStringAnsi::Strncpy(BindAddress, StringCast<ANSICHAR>(*InBindAddress).Get(), AddressLen+1);
		Info.iface = BindAddress;
	}

	Info.protocols = &Protocols[0];
	// no extensions
	Info.extensions = NULL;
	Info.gid = -1;
	Info.uid = -1;
	Info.options = LWS_SERVER_OPTION_ALLOW_LISTEN_SHARE;
	// tack on this object.
	Info.user = this;

	if (bSecure)
	{
		Info.options |= LWS_SERVER_OPTION_DO_SSL_GLOBAL_INIT;

		Info.ssl_cert_filepath = Certificates.GetCertificateFilePath();
		Info.ssl_private_key_filepath = Certificates.GetPrivateKeyFilePath();
	}
	const bool bUseIPv6 = EnumHasAnyFlags(NetworkProtocol, IWebSocketServer::ENetworkProtocol::IPv6);
	const bool bUseIPv4 = EnumHasAnyFlags(NetworkProtocol, IWebSocketServer::ENetworkProtocol::IPv4);

	if (!bUseIPv6)
	{
		Info.options |= LWS_SERVER_OPTION_DISABLE_IPV6;
	}
	else
	{
		// LWS_SERVER_OPTION_IPV6_V6ONLY_MODIFY tells lws to set IPV6_V6ONLY explicitly.
		Info.options |= LWS_SERVER_OPTION_IPV6_V6ONLY_MODIFY;
		if (!bUseIPv4)
		{
			// IPv6-only: set IPV6_V6ONLY=1 to reject IPv4 connections.
			Info.options |= LWS_SERVER_OPTION_IPV6_V6ONLY_VALUE;
		}
		// else: dual-stack (IPV6_V6ONLY=0) — IPv4 connections accepted via ::ffff: mapped addresses.
		// This is required on Windows where the default is IPV6_V6ONLY=1.
	}

	if (bEnableHttp && LwsHttpMounts != NULL)
	{
		Info.mounts = &LwsHttpMounts[0];
	}

	Context = lws_create_context(&Info);

	if (Context == NULL)
	{
		ServerPort = 0;
		delete[] Protocols;
		Protocols = NULL;
		delete[] LwsHttpMounts;
		LwsHttpMounts = NULL;
		return false; // couldn't create a server.
	}

	ConnectedCallBack = CallBack;
#endif
	return true;
}

void FWebSocketServer::SetFilterConnectionCallback(FWebSocketFilterConnectionCallback InFilterConnectionCallback)
{
	FilterConnectionCallback = MoveTemp(InFilterConnectionCallback);
}

void FWebSocketServer::Tick()
{
#if USE_LIBWEBSOCKET
	lws_service(Context, 0);
	lws_callback_on_writable_all_protocol(Context, &Protocols[0]);
#endif
}

FWebSocketServer::~FWebSocketServer()
{
#if USE_LIBWEBSOCKET

	if (Context)
	{
		lws_context* ExistingContext = Context;
		Context = NULL;
		lws_context_destroy(ExistingContext);
	}

	delete[] Protocols;
	Protocols = NULL;

	delete[] LwsHttpMounts;
	LwsHttpMounts = NULL;

	delete[] BindAddress;
	BindAddress = NULL;
#endif
}

FString FWebSocketServer::Info()
{
#if USE_LIBWEBSOCKET
	return FString::Printf(TEXT("%s:%i"), ANSI_TO_TCHAR(lws_canonical_hostname(Context)), ServerPort);
#else // ! USE_LIBWEBSOCKET -- i.e. HTML5 currently does not allow this...
	return FString(TEXT("NOT SUPPORTED"));
#endif
}

// callback.
#if USE_LIBWEBSOCKET
static int unreal_networking_server(
	struct lws*				  Wsi,
	enum lws_callback_reasons Reason,
	void*					  User,
	void*					  In,
	size_t					  Len)
{
	struct lws_context*	  Context = lws_get_context(Wsi);
	PerSessionDataServer* BufferInfo = (PerSessionDataServer*)User;
	FWebSocketServer*	  Server = (FWebSocketServer*)lws_context_user(Context);
	bool				  bRejectConnection = false;

	switch (Reason)
	{
		case LWS_CALLBACK_ESTABLISHED:
		{
			BufferInfo->Socket = new FWebSocket(Context, Wsi);
			BufferInfo->FragementationState = EFragmentationState::BeginFrame;
			Server->ConnectedCallBack.ExecuteIfBound(BufferInfo->Socket);
			lws_set_timeout(Wsi, NO_PENDING_TIMEOUT, 0);
		}
		break;

		case LWS_CALLBACK_RECEIVE:
			if (BufferInfo->Socket->Context == Context) // UE-74107 -- bandaid until this file is removed in favor of using LwsWebSocketsManager.cpp & LwsWebSocket.cpp
			{
				switch (BufferInfo->FragementationState)
				{
					case EFragmentationState::BeginFrame:
					{
						BufferInfo->FragementationState = EFragmentationState::MessageFrame;
						BufferInfo->FrameBuffer.Reset();

						// Fallthrough to read the message
					}
					case EFragmentationState::MessageFrame:
					{
						BufferInfo->FrameBuffer.Append((uint8*)In, Len);

						if (lws_is_final_fragment(Wsi))
						{
							BufferInfo->FragementationState = EFragmentationState::BeginFrame;
							if (!lws_frame_is_binary(Wsi))
							{
								BufferInfo->Socket->OnReceive(BufferInfo->FrameBuffer.GetData(), BufferInfo->FrameBuffer.Num());
							}
							else
							{
								BufferInfo->Socket->OnRawRecieve(BufferInfo->FrameBuffer.GetData(), BufferInfo->FrameBuffer.Num());
							}
						}

						break;
					}
					default:
						checkNoEntry();
						break;
				}
			}
			lws_set_timeout(Wsi, NO_PENDING_TIMEOUT, 0);
			break;

		case LWS_CALLBACK_SERVER_WRITEABLE:
			if (BufferInfo->Socket->Context == Context) // UE-68340 -- bandaid until this file is removed in favor of using LwsWebSocketsManager.cpp & LwsWebSocket.cpp
			{
				BufferInfo->Socket->OnRawWebSocketWritable(Wsi);

				// Note: This particular lws callback reason gets hit in both ws and http cases as it used to signal that the server is in a writeable state.
				// This means we only want to set an infinite timeout on genuine websocket connections, not http connections, otherwise they hang!
				lws_set_timeout(Wsi, NO_PENDING_TIMEOUT, 0);
			}
			break;
		case LWS_CALLBACK_CLIENT_CONNECTION_ERROR:
		{
			BufferInfo->Socket->ErrorCallBack.ExecuteIfBound();
		}
		break;
		case LWS_CALLBACK_CLOSED:
		{
			if (Server != nullptr)
			{
				bool bShuttingDown = Server->Context == NULL;
				if (!bShuttingDown && BufferInfo->Socket->Context == Context)
				{
					BufferInfo->Socket->OnClose();
				}
			}
			bRejectConnection = true;
		}
		break;
		case LWS_CALLBACK_WSI_DESTROY:
			break;
		case LWS_CALLBACK_PROTOCOL_DESTROY:
	#if PLATFORM_WINDOWS
		{
			// There is a bug with our version of the LWS library that keeps a socket open even if we destroy the context so we have to manually shut it down.
			lws_sockfd_type Fd = lws_get_socket_fd(Wsi);
			if (Fd != INVALID_SOCKET)
			{
				closesocket(Fd);
			}
		}
	#endif
		break;
		case LWS_CALLBACK_FILTER_PROTOCOL_CONNECTION:
		{
			if (Server && Server->FilterConnectionCallback.IsBound())
			{
				const int32 OriginHeaderLength = lws_hdr_total_length(Wsi, WSI_TOKEN_ORIGIN);

				FString OriginHeader;

				if (OriginHeaderLength != 0)
				{
					constexpr uint32 MaximumHeaderLength = 1024;
					if (OriginHeaderLength < MaximumHeaderLength)
					{
						char Origin[MaximumHeaderLength + 1];
						memset(Origin, 0, MaximumHeaderLength + 1);

						lws_hdr_copy(Wsi, Origin, OriginHeaderLength + 1, WSI_TOKEN_ORIGIN);
						OriginHeader = UTF8_TO_TCHAR(Origin);
					}
				}

				constexpr int32 IPAddressLength = 46; // INET6_ADDRSTRLEN — large enough for IPv6
				char			IPAddress[IPAddressLength];

				lws_get_peer_simple(Wsi, IPAddress, IPAddressLength);

				const FString ClientIP = UTF8_TO_TCHAR(IPAddress);

				if (Server->FilterConnectionCallback.Execute(OriginHeader, ClientIP) == EWebsocketConnectionFilterResult::ConnectionRefused)
				{
					bRejectConnection = true;
				}
			}

			break;
		}
	}

	// Check if http should be enabled or not, if so, use the in-built `lws_callback_http_dummy` which handles basic http requests
	if (Server != nullptr && Server->IsHttpEnabled())
	{
		return lws_callback_http_dummy(Wsi, Reason, User, In, Len);
	}

	return bRejectConnection ? 1 : 0;
}
#endif