// Copyright Epic Games, Inc. All Rights Reserved.
//
// Read http://lucumr.pocoo.org/2012/9/24/websockets-101/ for a nice intro to web sockets.
// This uses https://libwebsockets.org/trac/libwebsockets
#pragma once
#include "IWebSocketServer.h"
#include "WebSocketNetworkingPrivate.h"

class FWebSocketServer : public IWebSocketServer
{
public:
	FWebSocketServer() = default;

	//~ Begin IWebSocketServer interface
	virtual ~FWebSocketServer() override;
	virtual void	EnableHTTPServer(TArray<FWebSocketHttpMount> DirectoriesToServe, bool bSecure, const FWebSocketServerCertificates& Certificates) override;
	virtual bool	Init(uint32 Port, FWebSocketClientConnectedCallBack, FString BindAddress = TEXT("")) override;
	virtual void	SetFilterConnectionCallback(FWebSocketFilterConnectionCallback InFilterConnectionCallback) override;
	virtual void	SetNetworkProtocol(IWebSocketServer::ENetworkProtocol Protocol) override { NetworkProtocol = Protocol; }
	virtual void	Tick() override;
	virtual FString Info() override;
	//~ End IWebSocketServer interface

	bool IsHttpEnabled() const;

	// this was made public because of cross-platform build issues
public:
	/** Callback for a new websocket connection to the server */
	FWebSocketClientConnectedCallBack ConnectedCallBack;

	/** Handler called to filter incoming websocket connections. */
	FWebSocketFilterConnectionCallback FilterConnectionCallback;

	/** Internal libwebsocket context */
	WebSocketInternalContext* Context;

	/** Protocols serviced by this implementation */
	WebSocketInternalProtocol* Protocols;

	friend class FWebSocket;
	uint32 ServerPort;

private:
	bool bEnableHttp = false;

	IWebSocketServer::ENetworkProtocol NetworkProtocol = IWebSocketServer::ENetworkProtocol::IPv4;

	bool bSecure = false;

	FWebSocketServerCertificates Certificates;

	TArray<FWebSocketHttpMount> DirectoriesToServe;
	WebSocketInternalHttpMount* LwsHttpMounts = NULL;

	char* BindAddress = NULL;
};
