// Copyright Epic Games, Inc. All Rights Reserved.


#include "Net/DataChannel.h"

#include "PacketHandlers/StatelessConnectHandlerComponent.h"
#include "WebSocket.h"
#include "WebSocketConnection.h"

/*-----------------------------------------------------------------------------
Declarations.
-----------------------------------------------------------------------------*/

// Size of a UDP header.
#define IP_HEADER_SIZE     (20)
#define UDP_HEADER_SIZE    (IP_HEADER_SIZE+8)
#define WINSOCK_MAX_PACKET (512)

UWebSocketConnection::UWebSocketConnection(const FObjectInitializer& ObjectInitializer) :
Super(ObjectInitializer),
WebSocket(NULL)
{
}

void UWebSocketConnection::InitBase(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	// Pass the call up the chain
	Super::InitBase(InDriver, InSocket, InURL, InState,
		// Use the default packet size/overhead unless overridden by a child class
		InMaxPacket == 0 ? WINSOCK_MAX_PACKET : InMaxPacket,
		InPacketOverhead == 0 ? UDP_HEADER_SIZE : InPacketOverhead);

}

void UWebSocketConnection::InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	InitBase(InDriver, InSocket, InURL, InState,
		// Use the default packet size/overhead unless overridden by a child class
		InMaxPacket == 0 ? WINSOCK_MAX_PACKET : InMaxPacket,
		InPacketOverhead == 0 ? UDP_HEADER_SIZE : InPacketOverhead);

	// Figure out IP address from the host URL

	// Initialize our send bunch
	InitSendBuffer();
}

void UWebSocketConnection::InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	InitBase(InDriver, InSocket, InURL, InState,
		// Use the default packet size/overhead unless overridden by a child class
		InMaxPacket == 0 ? WINSOCK_MAX_PACKET : InMaxPacket,
		InPacketOverhead == 0 ? UDP_HEADER_SIZE : InPacketOverhead);

	// Initialize our send bunch
	InitSendBuffer();

	// This is for a client that needs to log in, setup ClientLoginState and ExpectedClientLoginMsgType to reflect that
	SetClientLoginState(EClientLoginState::LoggingIn);
	SetExpectedClientLoginMsgType(NMT_Hello);
}

void UWebSocketConnection::LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits)
{
	const uint8* DataToSend = reinterpret_cast<uint8*>(Data);

	// Process any packet modifiers
	if (Handler.IsValid() && !Handler->GetRawSend())
	{
		const ProcessedPacket ProcessedData = Handler->Outgoing(reinterpret_cast<uint8*>(Data), CountBits, Traits);

		if (!ProcessedData.bError)
		{
			DataToSend = ProcessedData.Data;
			CountBits = ProcessedData.CountBits;
		}
		else
		{
			CountBits = 0;
		}
	}

	uint32 CountBytes = FMath::DivideAndRoundUp(CountBits, 8);

	if (CountBits > (MaxPacket * 8))
	{
		UE_LOGF( LogNet, Warning, "UWebSocketConnection::LowLevelSend: CountBytes > MaxPacketSize! Count: %i, MaxPacket: %i %ls", CountBytes, MaxPacket, *Describe() );
	}

	bool bBlockSend = false;

#if !UE_BUILD_SHIPPING
	UE_LOGF(LogNetTraffic, VeryVerbose, "UWebSocketConnection::LowLevelSend: Address: %ls, CountBytes: %i", *LowLevelGetRemoteAddress(true), CountBytes);

	LowLevelSendDel.ExecuteIfBound((void*)DataToSend, CountBytes, bBlockSend);
#endif

	if (!bBlockSend && CountBytes > 0)
	{
		WebSocket->Send((uint8*)DataToSend, CountBytes);
	}
}

FString UWebSocketConnection::LowLevelGetRemoteAddress(bool bAppendPort)
{
	return WebSocket->RemoteEndPoint(bAppendPort);
}

FString UWebSocketConnection::LowLevelDescribe()
{
	return FString::Printf
		(
		TEXT(" remote=%s local=%s state: %s"),
		*WebSocket->RemoteEndPoint(true),
		*WebSocket->LocalEndPoint(true),
		LexToString(GetConnectionState())
		);
}

void UWebSocketConnection::SetWebSocket(INetworkingWebSocket* InWebSocket)
{
	WebSocket = InWebSocket; 
}

INetworkingWebSocket* UWebSocketConnection::GetWebSocket()
{
	return WebSocket;
}

void UWebSocketConnection::Tick(float DeltaSeconds)
{
	UNetConnection::Tick(DeltaSeconds);
	WebSocket->Tick();
}

void UWebSocketConnection::FinishDestroy()
{
	Super::FinishDestroy();
	if (WebSocket)
	{
#if USE_LIBWEBSOCKET
		static_cast<FWebSocket*>(WebSocket)->Context = NULL; // UE-68340;
#endif
		delete WebSocket; 
		WebSocket = NULL;
	}
}

void UWebSocketConnection::ReceivedRawPacket(void* Data,int32 Count)
{
#if !UE_BUILD_SHIPPING
	UE_LOGF(LogNetTraffic, VeryVerbose, "UWebSocketConnection::ReceivedRawPacket: Address: %ls, Count: %i", *LowLevelGetRemoteAddress(true), Count);
#endif

	if (Count == 0 ||   // nothing to process
		Driver == NULL) // connection closing
	{
		return;
	}

	uint8* DataRef = reinterpret_cast<uint8*>(Data);
	if ( bChallengeHandshake )
	{
		// Process all incoming packets.
		if (Driver->ConnectionlessHandler.IsValid() && Driver->StatelessConnectComponent.IsValid() && Driver->GetSocketSubsystem() != nullptr)
		{
			int32 Port;
			TSharedPtr<FInternetAddr> IncomingAddress = Driver->GetSocketSubsystem()->CreateInternetAddr();
			IncomingAddress->SetRawIp(WebSocket->GetRawRemoteAddr(Port));
			IncomingAddress->SetPort(Port);

			const ProcessedPacket UnProcessedPacket =
									Driver->ConnectionlessHandler->IncomingConnectionless(IncomingAddress, DataRef, Count);
	
			TSharedPtr<StatelessConnectHandlerComponent> StatelessConnect = Driver->StatelessConnectComponent.Pin();

			bool bRestartedHandshake = false;

			if (!UnProcessedPacket.bError && StatelessConnect->HasPassedChallenge(IncomingAddress, bRestartedHandshake) &&
					!bRestartedHandshake)
			{
				UE_LOGF(LogNet, Log, "Server accepting post-challenge connection from: %ls", *(IncomingAddress->ToString(false)));
				// Set the initial packet sequence from the handshake data
				if (StatelessConnectComponent.IsValid())
				{
					int32 ServerSequence = 0;
					int32 ClientSequence = 0;
					StatelessConnect->GetChallengeSequence(ServerSequence, ClientSequence);
					InitSequence(ClientSequence, ServerSequence);
				}

				if (Handler.IsValid())
				{
					Handler->BeginHandshaking();
				}

				bChallengeHandshake = false; // i.e. bPassedChallenge
				UE_LOGF(LogNet, Warning, "UWebSocketConnection::bChallengeHandshake: %ls", *LowLevelDescribe());
				Count = FMath::DivideAndRoundUp(UnProcessedPacket.CountBits, 8);
				if (Count > 0)
				{
					DataRef = UnProcessedPacket.Data;
				}
				else
				{
					return; // NO FURTHER DATA TO PROCESS
				}
			}
			else
			{
				// WARNING: if here, it might be during (bInitialConnect) - which needs to be processed (ReceivedRawPacket)
				//return;
			}
		}
	}

	UNetConnection::ReceivedRawPacket(DataRef,Count);
}
