// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "WebSocketNetworkingDelegates.h"

class INetworkingWebSocket
{
public:
	virtual ~INetworkingWebSocket() = default;

	/************************************************************************/
	/* Set various callbacks for Socket Events                              */
	/************************************************************************/
	virtual void SetConnectedCallBack(FWebSocketInfoCallBack CallBack) = 0;
	virtual void SetErrorCallBack(FWebSocketInfoCallBack CallBack) = 0;
	virtual void SetReceiveCallBack(FWebSocketPacketReceivedCallBack CallBack) = 0;
	virtual void SetSocketClosedCallBack(FWebSocketInfoCallBack CallBack) = 0;

	/** Send raw data to remote end point. */
	virtual bool Send(const uint8* Data, uint32 Size, bool bPrependSize = true) = 0;

	/** service libwebsocket.			   */
	virtual void Tick() = 0;
	/** service libwebsocket until outgoing buffer is empty */
	virtual void Flush() = 0;

	/** Helper functions to describe end points. */
	virtual TArray<uint8> GetRawRemoteAddr(int32& OutPort) = 0;
	virtual FString RemoteEndPoint(bool bAppendPort) = 0;
	virtual FString LocalEndPoint(bool bAppendPort) = 0;
	virtual struct sockaddr_in* GetRemoteAddr() = 0;
	virtual TArray<FString> GetUrlArgs() = 0;
	virtual FString GetUrlArgByName(const FString& Name) = 0;
};
