// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"


class IWebSocketServer;
class INetworkingWebSocket;

/* 
 * Interface for the WebSocketNetworking module. 
 */
class IWebSocketNetworkingModule
	: public IModuleInterface
{
public:
	virtual ~IWebSocketNetworkingModule() = default;

	/**
	 * Create a WebSocket server.
	 *
	 * @return A new WebSocket server, or nullptr if the server couldn't be created.
	 */
	virtual TUniquePtr<IWebSocketServer> CreateServer() = 0;

	/**
	 * Create a WebSocket client connection.
	 *
	 * @return A new WebSocket client connection, or nullptr if the connection couldn't be created.
	 */
	virtual TSharedPtr<INetworkingWebSocket> CreateConnection(const class FInternetAddr& ServerAddress) = 0;
};
