// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "WebSocketNetworkingDelegates.h"
#include <string>

/**
 * Represents a directory on disk that can be mapped to a url route and served.
 *
 * For example we may wish to map `C:/www/Public` to `/` of our webserver.
 **/
class FWebSocketHttpMount
{
public:
	/**
	 * Sets the absolute path on disk to the directory we wish to serve.
	 * @param InPathOnDisk The absolute path to the directory.
	 */
	void SetPathOnDisk(const FString& InPathOnDisk) { PathOnDisk = std::string(TCHAR_TO_ANSI(*InPathOnDisk)); }

	/**
	 * Sets The web url path to use for this mount, e.g. /images .
	 * @param InWebPath The web url path we will map the directory to, e.g. /images.
	 */
	void SetWebPath(const FString& InWebPath) { WebPath = std::string(TCHAR_TO_ANSI(*InWebPath)); }

	/**
	 * Sets a file to serve when the root web path is requested.
	 * @param InDefaultFile The file to serve when the root web path is request, e.g. index.html.
	 */
	void SetDefaultFile(const FString& InDefaultFile) { DefaultFile = std::string(TCHAR_TO_ANSI(*InDefaultFile)); }

	const char* GetPathOnDisk() { return PathOnDisk.c_str(); }
	const char* GetWebPath() { return WebPath.c_str(); }
	const char* GetDefaultFile() { return DefaultFile.c_str(); }
	bool		HasDefaultFile() { return DefaultFile.empty(); }

private:
	// Note: The below members are std::string purposefully as lws requires char* strings and FString may not be char*.

	/* The absolute path on disk to directory we wish to serve. */
	std::string PathOnDisk;

	/* The web url path to use for this mount, e.g. /images */
	std::string WebPath;

	/* When the root of the `WebPath` is requested, without a file, we can serve this file, e.g. index.html */
	std::string DefaultFile = "index.html";
};

class FWebSocketServerCertificates
{
public:
	/**
	 * Sets the absolute path on disk to the file containing the SSL certificate.
	 * @param InCertificateFilePath The absolute path to the file.
	 */
	void SetCertificateFilePath(const FString& InCertificateFilePath) { CertificateFilePath = std::string(TCHAR_TO_ANSI(*InCertificateFilePath)); }

	/**
	 * Sets the absolute path on disk to the file containing the SSL private key.
	 * @param InPrivateKeyFilePath The absolute path to the file.
	 */
	void SetPrivateKeyFilePath(const FString& InPrivateKeyFilePath) { PrivateKeyFilePath = std::string(TCHAR_TO_ANSI(*InPrivateKeyFilePath)); }

	const char* GetCertificateFilePath() { return CertificateFilePath.c_str(); }
	const char* GetPrivateKeyFilePath() { return PrivateKeyFilePath.c_str(); }

private:
	// Note: The below members are std::string purposefully as lws requires char* strings and FString may not be char*.

	// The absolute path to the certificate file
	std::string CertificateFilePath;

	// The absolute path to the private key file
	std::string PrivateKeyFilePath;
};

class IWebSocketServer
{
public:
	/** Bitmask of network protocols the server should listen on. */
	enum class ENetworkProtocol : uint8
	{
		None = 0,
		IPv4 = 1 << 0,
		IPv6 = 1 << 1,
	};

	virtual ~IWebSocketServer() = default;

	/**
	 * Extends the websocket server to additionally serve static content from on-disk directories using HTTP or HTTPS.
	 * These HTTP or HTTPS requests will use the same port as the one specified for websocket use in `Init`.
	 * Note: This function will not start the webserver, call `Init` to start the websocket/webserver.
	 * @param DirectoriesToServe The directories for the Webserver to serve.
	 * @param bSecure Should the HTTP server serve the directories over HTTPS
	 * @param Certificates FWebSocketServerCertificates containing paths to SSL certificates required when serving over HTTPS
	 */
	virtual void EnableHTTPServer(TArray<FWebSocketHttpMount> DirectoriesToServe, bool bSecure = false, const FWebSocketServerCertificates& Certificates = FWebSocketServerCertificates()) = 0;

	/**
	 * Initialize the server and start listening for messages.
	 * @param Port the port to handle websockets messages on.
	 * @param ClientConnectedCallback the handler called when a client has connected.
	 * @param BindAddress Which address the server should bind to. If left empty, the server will bind to all interfaces.
	 * @return whether the initialization was successful.
	 */
	virtual bool Init(uint32 Port, FWebSocketClientConnectedCallBack ClientConnectedCallback, FString BindAddress = TEXT("")) = 0;

	/**
	 * Set a filter callback, used to reject connections based on the origin header.
	 * @param FilterConnectionCallback handler called to filter the connection.
	 */
	virtual void SetFilterConnectionCallback(FWebSocketFilterConnectionCallback InFilterConnectionCallback) = 0;

	/**
	 * Set which network protocols the server should listen on.
	 * Must be called before Init(). Default: IPv4 only.
	 *   IPv4       - listen on 0.0.0.0 (IPv4 only)
	 *   IPv6       - listen on :: (IPv6 only, IPV6_V6ONLY=1)
	 *   IPv4|IPv6  - listen on :: (dual-stack, IPV6_V6ONLY=0)
	 */
	virtual void SetNetworkProtocol(ENetworkProtocol Protocol) {}

	/** Tick the server. */
	virtual void Tick() = 0;

	/** Describe this libwebsocket server */
	virtual FString Info() = 0;
};

ENUM_CLASS_FLAGS(IWebSocketServer::ENetworkProtocol);
