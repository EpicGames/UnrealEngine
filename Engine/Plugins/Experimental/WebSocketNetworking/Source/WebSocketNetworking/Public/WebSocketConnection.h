// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/NetConnection.h"
#include "WebSocketConnection.generated.h"

#define UE_API WEBSOCKETNETWORKING_API


UCLASS(MinimalAPI, transient, config = Engine)
class UWebSocketConnection : public UNetConnection
{
	GENERATED_UCLASS_BODY()

	class INetworkingWebSocket* WebSocket;

	//~ Begin NetConnection Interface
	UE_API virtual void InitBase(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void InitRemoteConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, const class FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void InitLocalConnection(UNetDriver* InDriver, class FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	UE_API virtual void LowLevelSend(void* Data, int32 CountBits, FOutPacketTraits& Traits) override;
	UE_API FString LowLevelGetRemoteAddress(bool bAppendPort = false) override;
	UE_API FString LowLevelDescribe() override;
	UE_API virtual void Tick(float DeltaSeconds) override;
	UE_API virtual void FinishDestroy();
	UE_API virtual void ReceivedRawPacket(void* Data,int32 Count);
	//~ End NetConnection Interface


	UE_API void SetWebSocket(INetworkingWebSocket* InWebSocket);
	UE_API INetworkingWebSocket* GetWebSocket();

	bool bChallengeHandshake = false;
};

#undef UE_API
