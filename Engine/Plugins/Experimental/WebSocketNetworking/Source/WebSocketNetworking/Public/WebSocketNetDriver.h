// Copyright Epic Games, Inc. All Rights Reserved.
//
// websocket based implementation of the net driver
//

#pragma once

#include "CoreMinimal.h"
#include "Engine/NetDriver.h"
#include "WebSocketNetDriver.generated.h"

#define UE_API WEBSOCKETNETWORKING_API

class INetworkingWebSocket;
class IWebSocketServer;

UCLASS(MinimalAPI, transient, config = Engine)
class UWebSocketNetDriver : public UNetDriver
{
	GENERATED_UCLASS_BODY()

	/** WebSocket server port*/
	UPROPERTY(Config)
	int32 WebSocketPort;

	//~ Begin UNetDriver Interface.
	UE_API virtual bool IsAvailable() const override;
	UE_API virtual bool InitBase(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, FString& Error) override;
	UE_API virtual bool InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error) override;
	UE_API virtual bool InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error) override;
	UE_API virtual void TickDispatch(float DeltaTime) override;
	UE_API virtual void LowLevelSend(TSharedPtr<const FInternetAddr> Address, void* Data, int32 CountBits, FOutPacketTraits& Traits) override;
	UE_API virtual FString LowLevelGetNetworkNumber() override;
	UE_API virtual void LowLevelDestroy() override;
	UE_API virtual bool IsNetResourceValid(void) override;

	// stub implementation because for websockets we don't use any underlying socket sub system.
	UE_API virtual class ISocketSubsystem* GetSocketSubsystem() override;

	//~ End UNetDriver Interface.

	//~ Begin FExec Interface
#if UE_ALLOW_EXEC_COMMANDS
	UE_API virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar = *GLog) override;
#endif
	//~ End FExec Interface

	/**
	* Exec command handlers
	*/
	UE_API bool HandleSocketsCommand(const TCHAR* Cmd, FOutputDevice& Ar, UWorld* InWorld);

	/** @return connection to server */
	UE_API class UWebSocketConnection* GetServerConnection();

	/************************************************************************/
	/* IWebSocketServer														*/
	/************************************************************************/

	IWebSocketServer* WebSocketServer;

	/******************************************************************************/
	/* Callback Function for New Connection from a client is accepted by this server   */
	/******************************************************************************/

	UE_API void OnWebSocketClientConnected(INetworkingWebSocket*); // to the server.


	/************************************************************************/
	/* Callback Function for when this client Connects to the server				*/
	/************************************************************************/

	UE_API void OnWebSocketServerConnected(); // to the client.


};

#undef UE_API
