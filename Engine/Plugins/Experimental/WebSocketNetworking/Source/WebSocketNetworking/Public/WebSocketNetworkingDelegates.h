// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class INetworkingWebSocket;

enum class EWebsocketConnectionFilterResult : uint8
{
	ConnectionAccepted,
	ConnectionRefused
};

DECLARE_DELEGATE(FWebSocketInfoCallBack);
DECLARE_DELEGATE_TwoParams(FWebSocketPacketReceivedCallBack, void* /*Data*/, int32 /*Data Size*/);
DECLARE_DELEGATE_OneParam(FWebSocketClientConnectedCallBack, INetworkingWebSocket* /*Socket*/);
DECLARE_DELEGATE_RetVal_TwoParams(EWebsocketConnectionFilterResult, FWebSocketFilterConnectionCallback, FString /*Origin*/, FString /*ClientIP*/);
						  
