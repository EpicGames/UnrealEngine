// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class WebSocketNetworking  : ModuleRules
	{
		public WebSocketNetworking(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
                {
					"Engine"
                }
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"NetCore",
					"EngineSettings",
					"ImageCore",
					"Sockets",
					"PacketHandler",
					"OpenSSL",
					"libWebSockets",
					"zlib"
				}
			);

			PublicDefinitions.Add("USE_LIBWEBSOCKET=1");
		}
	}
}
