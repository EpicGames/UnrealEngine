{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Experimental WebSocket Networking Plugin",
	"Description" : "WebSocket Networking - NOTE: MUST disable all other existing NetDriverDefinitions in order to use WebSocketNetDriver. ALSO: MUST disable all PackHandlerComponents not supported by HTML5/Websockets (e.g. SteamAuthComponentModuleInterface)",
	"Category" : "Misc",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "WebSocketNetworking",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Mac",
				"Win64",
				"Linux"
			]
		}
	]
}
